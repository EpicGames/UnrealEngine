// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DiffAssetBulkData : ModuleRules
{
	public DiffAssetBulkData(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(new string[] {
			"ApplicationCore",
			"AssetRegistry",
			"Core", 
			"CoreUObject",
			"Json",
			"Projects", 
		});
	}
}
