// Copyright Epic Games, Inc. All Rights Reserved.

#include "RequiredProgramMainCPPInclude.h"

#include "Algo/Sort.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "IO/IoDispatcher.h"
#include "IO/IoHash.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

IMPLEMENT_APPLICATION(DiffAssetBulkData, "DiffAssetBulkData");


DEFINE_LOG_CATEGORY_STATIC(LogDiffAssetBulk, Display, All);

/**
 * Diff Asset Bulk Data
 * 
 * This loads two asset registries newer than FAssetRegistryVersion::AddedChunkHashes,
 * and attempts to find the reason for bulk data differences.
 * 
 * First, it finds what bulk datas changed by using the hash of the bulk data,
 * then it uses "Diff Tags" to try and determine at what point during the derived data
 * build the change occurred.
 * 
 * 
 * Diff Tags
 * 
 * Diff Tags are cook tags added during the cook process using Ar.CookContext()->CookTagList() (see CookTagList.h)
 * and are of the form "Cook_Diff_##_Key":
 *
 * 		- "Cook_": 	Added automatically by the the cook tag system.
 * 		- "Diff_": 	Identifies the tag as a diff tag.
 * 		- "##":		Specifies where in the build process the tag represents (Ordering).
 * 		- "_Key":	Descriptive text for the tag.
 * 
 * If a bulk data difference is found, the diff tags are checked for differences in order, and the first
 * diff tag that changed is assigned the "blame" for the change under the assumption that later
 * tags will necessarily change as a result of the earlier change.
 * 
 * If diff tags are present for the asset and none of the diff tags changed, then it is assumed that a build determinism 
 * issue has caused the change.
 *
 */

/**
 *  The list of known cook diff tags - this is just used to provide explanations in the output for the reader.
 */
static struct FBuiltinDiffTagHelp {const TCHAR* TagName; const TCHAR* TagHelp;} GBuiltinDiffTagHelp[] = 
{
	{TEXT("Cook_Diff_20_Tex2D_CacheKey"), TEXT("Texture settings or referenced data changed (DDC2)")},
	{TEXT("Cook_Diff_20_Tex2D_DDK"), TEXT("Texture settings or referenced data changed (DDC1)")},
	{TEXT("Cook_Diff_10_Tex2D_Source"), TEXT("Texture source data changed")}
};


static int32 RunDiffAssetBulkData()
{
	FString BaseFileName, CurrentFileName;
	const TCHAR* CmdLine = FCommandLine::Get();
	if (FParse::Value(CmdLine, TEXT("Base="), BaseFileName) == false ||
		FParse::Value(CmdLine, TEXT("Current="), CurrentFileName) == false)
	{
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "Diff Asset Bulk Data");
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "Loads two development asset registries and finds all bulk data changes, and tries to find why");
		UE_LOGF(LogDiffAssetBulk, Display, "the bulk data changed. Development asset registries are in the cooked /Metadata directory.");
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "Parameters:");
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "    -Base=<path/to/file>              Base Development Asset Registry (Required)");
		UE_LOGF(LogDiffAssetBulk, Display, "    -Current=<path/to/file>           New Development Asset Registry (Required)");
		UE_LOGF(LogDiffAssetBulk, Display, "    -Optional                         Evaluate Optional bulk data changes instead.");
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListMixed                        Show the list of changed packages with assets that have matching");
		UE_LOGF(LogDiffAssetBulk, Display, "                                      blame tags, but also assets without.");
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListDeterminism                  Show the list of changed packages with assets that have matching");
		UE_LOGF(LogDiffAssetBulk, Display, "                                      blame tags.");		
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListBlame=<blame tag>            Show the list of assets that changed due to a specific blame");
		UE_LOGF(LogDiffAssetBulk, Display, "                                      tag or \"All\" to list all changed assets with known blame.");
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListUnrepresented                Show the list of packages where a representative asset couldn't be found."); 
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListNoBlame=<class>              Show the list of assets that changed for a specific class, or \"All\"");
		UE_LOGF(LogDiffAssetBulk, Display, "    -ListCSV=<filename>               Write all changed packages to the given CSV file.");
		return 1;
	}

	bool bEvaluateOptional = FParse::Param(CmdLine, TEXT("Optional"));
	bool bListMixed = FParse::Param(CmdLine, TEXT("ListMixed"));
	bool bListDeterminism = FParse::Param(CmdLine, TEXT("ListDeterminism"));
	bool bListUnrepresented = FParse::Param(CmdLine, TEXT("ListUnrepresented"));
	FString ListBlame;
	FParse::Value(CmdLine, TEXT("ListBlame="), ListBlame);
	FString ListNoBlame;
	FParse::Value(CmdLine, TEXT("ListNoBlame="), ListNoBlame);

	FString ListCSV;
	TUniquePtr<FArchive> ChangedCSVAr;
	TUniquePtr<FArchive> NewCSVAr;
	TUniquePtr<FArchive> MovedCSVAr;
	TUniquePtr<FArchive> DeletedCSVAr;
	if (FParse::Value(CmdLine, TEXT("ListCSV="), ListCSV))
	{
		FString Extension = FPaths::GetExtension(ListCSV);
		FString Base = FPaths::ChangeExtension(ListCSV, TEXT(""));

		ChangedCSVAr.Reset(IFileManager::Get().CreateFileWriter(*(Base + TEXT("Changed.") + Extension), 0));
		if (!ChangedCSVAr)
		{
			UE_LOGF(LogDiffAssetBulk, Error, "Unable to open output CSV file: %ls", *(Base + TEXT("Changed.") + Extension));
			return false;
		}
		ChangedCSVAr->Logf(TEXT("Blame, Class, PackageName, BlameBefore, BlameAfter, OldCompressedSize, NewCompressedSize, OldUncompressedSize, NewUncompressedSize"));

		NewCSVAr.Reset(IFileManager::Get().CreateFileWriter(*(Base + TEXT("New.") + Extension), 0));
		if (!NewCSVAr)
		{
			UE_LOGF(LogDiffAssetBulk, Error, "Unable to open output CSV file: %ls", *(Base + TEXT("New.") + Extension));
			return false;
		}
		NewCSVAr->Logf(TEXT("Class, PackageName"));

		DeletedCSVAr.Reset(IFileManager::Get().CreateFileWriter(*(Base + TEXT("Deleted.") + Extension), 0));
		if (!DeletedCSVAr)
		{
			UE_LOGF(LogDiffAssetBulk, Error, "Unable to open output CSV file: %ls", *(Base + TEXT("Deleted.") + Extension));
			return false;
		}
		DeletedCSVAr->Logf(TEXT("Class, PackageName"));

		MovedCSVAr.Reset(IFileManager::Get().CreateFileWriter(*(Base + TEXT("Moved.") + Extension), 0));
		if (!MovedCSVAr)
		{
			UE_LOGF(LogDiffAssetBulk, Error, "Unable to open output CSV file: %ls", *(Base + TEXT("Moved.") + Extension));
			return false;
		}
		MovedCSVAr->Logf(TEXT("Class, PackageName"));
	}

	// Convert the static init help text to a map
	TMap<FName, const TCHAR*> BuiltinDiffTagHelpMap;
	for (FBuiltinDiffTagHelp& DiffTagHelp : GBuiltinDiffTagHelp)
	{
		BuiltinDiffTagHelpMap.Add(DiffTagHelp.TagName, DiffTagHelp.TagHelp);
	}

	FAssetRegistryState BaseState, CurrentState;
	FAssetRegistryVersion::Type BaseVersion, CurrentVersion;
	UE_LOGF(LogDiffAssetBulk, Display, "Loading Base... (%ls)", *BaseFileName);
	if (FAssetRegistryState::LoadFromDisk(*BaseFileName, FAssetRegistryLoadOptions(), BaseState, &BaseVersion) == false)
	{
		UE_LOGF(LogDiffAssetBulk, Error, "Failed load base (%ls)", *BaseFileName);
		return 1;
	}
	UE_LOGF(LogDiffAssetBulk, Display, "Loading Current... (%ls)", *CurrentFileName);
	if (FAssetRegistryState::LoadFromDisk(*CurrentFileName, FAssetRegistryLoadOptions(), CurrentState, &CurrentVersion) == false)
	{
		UE_LOGF(LogDiffAssetBulk, Error, "Failed load current (%ls)", *CurrentFileName);
		return 1;
	}

	//
	// The cook process adds the hash for almost all iochunks to the asset registry - 
	// so as long as both asset registries have that data, we get what we want.
	//
	if (BaseVersion < FAssetRegistryVersion::AddedChunkHashes)
	{
		UE_LOGF(LogDiffAssetBulk, Error, "Base asset registry version is too old (%d, need %d)", BaseVersion, FAssetRegistryVersion::AddedChunkHashes);
		return 1;
	}
	if (CurrentVersion < FAssetRegistryVersion::AddedChunkHashes)
	{
		UE_LOGF(LogDiffAssetBulk, Error, "Current asset registry version is too old (%d, need %d)", CurrentVersion, FAssetRegistryVersion::AddedChunkHashes);
		return 1;
	}

	const TMap<FName, const FAssetPackageData*>& BasePackages = BaseState.GetAssetPackageDataMap();
	const TMap<FName, const FAssetPackageData*>& CurrentPackages = CurrentState.GetAssetPackageDataMap();

	struct FIteratedPackage
	{
		FName Name = NAME_None;
		const FAssetPackageData* Base = nullptr;
		const FAssetPackageData* Current = nullptr;
		FIteratedPackage() = default;
		FIteratedPackage(FName _Name, const FAssetPackageData* _Base, const FAssetPackageData* _Current) :
			Name(_Name),
			Base(_Base),
			Current(_Current) {}

	};
	TArray<FIteratedPackage> UnionedPackages;

	uint64 CurrentTotalSize = 0;
	uint64 BaseTotalSize = 0;

	{
		for (const TPair<FName, const FAssetPackageData*>& NamePackageDataPair : BasePackages)
		{
			const FAssetPackageData* Current = CurrentState.GetAssetPackageData(NamePackageDataPair.Key);

			const FAssetData* BaseMIAsset = UE::AssetRegistry::GetMostImportantAsset(BaseState.CopyAssetsByPackageName(NamePackageDataPair.Key), UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
			uint64 BaseCompressedSize = 0;
			if (BaseMIAsset && BaseMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, BaseCompressedSize))
			{
				BaseTotalSize += BaseCompressedSize;
			}

			UnionedPackages.Emplace(FIteratedPackage(NamePackageDataPair.Key, NamePackageDataPair.Value, Current));
		}

		for (const TPair<FName, const FAssetPackageData*>& NamePackageDataPair : CurrentPackages)
		{
			const FAssetPackageData* Base = BaseState.GetAssetPackageData(NamePackageDataPair.Key);

			const FAssetData* CurrentMIAsset = UE::AssetRegistry::GetMostImportantAsset(CurrentState.CopyAssetsByPackageName(NamePackageDataPair.Key), UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
			uint64 CurrentCompressedSize = 0;
			if (CurrentMIAsset && CurrentMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CurrentCompressedSize))
			{
				CurrentTotalSize += CurrentCompressedSize;
			}

			if (Base == nullptr)
			{
				UnionedPackages.Emplace(FIteratedPackage(NamePackageDataPair.Key, nullptr, NamePackageDataPair.Value));
			}
		}
	}

	// Now we need to see what changed.
	//
	// This whole thing assumes that the index parameter of CreateIoChunkId is always 0. This is likely not going
	// to be true with FDerivedData, once that gets turned on, but should be easy to update when the time comes.
	//

	// Save off what hashes got deleted so we can try to find packages that moved and report those separately.
	TMap<FIoHash, TArray<FName, TInlineAllocator<1>>> DeletedChunkPackagesByHash;

	TSet<FName> PackagesWithChangedChunks;
	TSet<FName> PackagesWithDeletedChunks;
	TMap<FName, TArray<FIoHash, TInlineAllocator<1>>> PackagesWithNewChunks;

	auto ShouldProcessChunk = [bEvaluateOptional](const FIoChunkId& ChunkId)
	{
		if (ChunkId.GetChunkType() != EIoChunkType::BulkData &&
			ChunkId.GetChunkType() != EIoChunkType::OptionalBulkData &&
			ChunkId.GetChunkType() != EIoChunkType::MemoryMappedBulkData)
		{
			return false;
		}

		bool bIsOptional = ChunkId.GetChunkType() == EIoChunkType::OptionalBulkData;
		if (bEvaluateOptional)
		{
			return bIsOptional;
		}
		return !bIsOptional;
	};

	struct FPackageSizes
	{
		uint64 BaseCompressedSize = 0;
		uint64 CurrentCompressedSize = 0;
		uint64 BaseUncompressedSize = 0;
		uint64 CurrentUncompressedSize = 0;
	};
	uint64 TotalChangedSize = 0;
	TMap < FName /* PackageName */, FPackageSizes> PackageSizes;
	for (const FIteratedPackage& IteratedPackage : UnionedPackages)
	{
		const FAssetPackageData* BasePackage = IteratedPackage.Base;
		const FAssetPackageData* CurrentPackage = IteratedPackage.Current;


		// Get the size change.
		// IoStoreUtilities puts the size of the package on the most important asset
		const FAssetData* BaseMIAsset = UE::AssetRegistry::GetMostImportantAsset(BaseState.CopyAssetsByPackageName(IteratedPackage.Name), UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
		const FAssetData* CurrentMIAsset = UE::AssetRegistry::GetMostImportantAsset(CurrentState.CopyAssetsByPackageName(IteratedPackage.Name), UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);

		FPackageSizes& Sizes = PackageSizes.Add(IteratedPackage.Name);

		if (BaseMIAsset)
		{
			BaseMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, Sizes.BaseCompressedSize);
			BaseMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkSizeFName, Sizes.BaseUncompressedSize);
		}
		if (CurrentMIAsset)
		{
			CurrentMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, Sizes.CurrentCompressedSize);
			CurrentMIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkSizeFName, Sizes.CurrentUncompressedSize);
		}

		if (BasePackage)
		{
			for (const TPair<FIoChunkId, FIoHash>& ChunkHashPair : BasePackage->ChunkHashes)
			{
				if (!ShouldProcessChunk(ChunkHashPair.Key))
					continue;

				const FIoHash* CurrentHash = nullptr;
				if (CurrentPackage)
				{
					CurrentHash = CurrentPackage->ChunkHashes.Find(ChunkHashPair.Key);
				}

				if (CurrentHash == nullptr)
				{
					PackagesWithDeletedChunks.Add(IteratedPackage.Name);
					DeletedChunkPackagesByHash.FindOrAdd(ChunkHashPair.Value).Add(IteratedPackage.Name);
					continue;
				}

				if (*CurrentHash != ChunkHashPair.Value)
				{
					PackagesWithChangedChunks.Add(IteratedPackage.Name);

					// All we can really do here is assume the entire package gets resent, which is not likely
					// in the general case, but it _is_ reasonably likely in the cases where a package's bulk data changes,
					// which happens to be what we select on.
					// The counter argument is that it's possible that the bulk data is Very Large (i.e. multiple compression blocks), and only
					// one block out of the entire thing changed.
					if (BaseMIAsset && CurrentMIAsset)
					{
						TotalChangedSize += Sizes.CurrentCompressedSize;
					}
				}
			}
		}

		if (CurrentPackage)
		{
			for (const TPair<FIoChunkId, FIoHash>& ChunkHashPair : CurrentPackage->ChunkHashes)
			{
				if (!ShouldProcessChunk(ChunkHashPair.Key))
				{
					continue;
				}

				if (!BasePackage ||
					BasePackage->ChunkHashes.Contains(ChunkHashPair.Key) == false)
				{
					PackagesWithNewChunks.FindOrAdd(IteratedPackage.Name).Add(ChunkHashPair.Value);
				}
			}
		}
	}

	TMap<FName, FName> MovedPackagesFromTo;
	
	// Look over the new packages - if any of them have exact matching entries in the deleted list,
	// then we assume it's a moved chunk and remove it from the new/delete lists.
	for (const TPair<FName, TArray<FIoHash, TInlineAllocator<1>>>& PackageHashesPair : PackagesWithNewChunks)
	{
		// Make sure all chunks we know about moved from the same place. We expect this to be only 1 for now, so warn on it.
		FName MovedFrom = NAME_None;
		for (const FIoHash& NewHash : PackageHashesPair.Value)
		{
			const TArray<FName, TInlineAllocator<1>>* PackagesThatHadThisChunk = DeletedChunkPackagesByHash.Find(NewHash);
			if (PackagesThatHadThisChunk == nullptr ||
				PackagesThatHadThisChunk->Num() == 0)
			{
				MovedFrom = NAME_None;
				break;
			}

			// Due to duplication we could theoretically have the exact same bulk data in a bunch of
			// different packages, so we consider it a move if it's in any of them. This could fail
			// if there were multiple chunks where one came from one package and the other came from a different one,
			// seems unlikely.
			if (MovedFrom.IsNone())
			{
				// Grab the first one...
				MovedFrom = (*PackagesThatHadThisChunk)[0];
			}
			else
			{
				bool bFound = false;
				for (const FName& PackageThatHadThisChunk : (*PackagesThatHadThisChunk))
				{
					if (MovedFrom == PackageThatHadThisChunk)
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					MovedFrom = NAME_None;
					break;
				}
			}
		}

		if (MovedFrom.IsNone())
		{
			continue; // Not moved - actual new package.
		}

		// We also only allow path moves - this is because it's not uncommon for folks to duplicate something like a mesh
		// and change the material and this can confuse our hash matching. 
		// However, if it's a _Generated_ package we actually want to know because it might be an issue with the stability
		// of the generator.
		TStringBuilder<64> MovedFromStr;
		MovedFromStr << MovedFrom;
		if (!FCString::Stristr(*MovedFromStr, TEXT("_GENERATED_")))
		{
			// it's not generated, so make sure the name matches.
			TStringBuilder<64> MovedToStr;
			MovedToStr << PackageHashesPair.Key;

			const TCHAR* MovedFromShortName = FCString::Strrchr(*MovedFromStr, TEXT('/'));
			if (MovedFromShortName)
			{
				MovedFromShortName++;
			}
			const TCHAR* MovedToShortName = FCString::Strrchr(*MovedToStr, TEXT('/'));
			if (MovedToShortName)
			{
				MovedToShortName++;
			}

			// If we have short names and they are different, we assume it's not an actual move.
			if (MovedFromShortName && MovedToShortName && FCString::Stricmp(MovedFromShortName, MovedToShortName))
			{
				continue;
			}
		}

		if (MovedPackagesFromTo.Contains(MovedFrom))
		{
			UE_LOGF(LogDiffAssetBulk, Display, "Package %ls appears to have moved twice. Perhaps duplicated multiple times and original deleted? Or Material change?", *MovedFromStr);
			UE_LOGF(LogDiffAssetBulk, Display, "    Existing: %ls", *WriteToString<64>(*MovedPackagesFromTo.Find(MovedFrom)));
			UE_LOGF(LogDiffAssetBulk, Display, "         New: %ls", *WriteToString<64>(PackageHashesPair.Key));
			continue;
		}
		MovedPackagesFromTo.Add(MovedFrom, PackageHashesPair.Key);
	}

	// Done with this, empty it so it's obvious if we try to use it.
	DeletedChunkPackagesByHash.Empty();

	// Once we have the list of moved packages, remove them from the deleted/new lists
	for (const TPair<FName, FName>& MovedPackageFromTo : MovedPackagesFromTo)
	{
		if (!PackagesWithNewChunks.Remove(MovedPackageFromTo.Value))
		{
			UE_LOGF(LogDiffAssetBulk, Warning, "Unable to remove moved package %ls from the new list", *WriteToString<64>(MovedPackageFromTo.Value));
		}
		if (!PackagesWithDeletedChunks.Remove(MovedPackageFromTo.Key))
		{
			UE_LOGF(LogDiffAssetBulk, Warning, "Unable to remove moved package %ls from the deleted list", *WriteToString<64>(MovedPackageFromTo.Key));
		}
	}

	//
	// We know what bulk datas *packages* changed. Try and see if any of the assets in the package have
	// diff blame tags for us to determine cause. _usually_ there's one asset per package, but it's definitely
	// possible to have more. Additionally _usually_ there's a good single candidate for assigning the data
	// cost, however it is possible to have e.g. an importer create a lot of assets in a single package that
	// all add bulk data to the package.
	// 
	// Once we have FDerivedData we might be able to keep what data belongs to which asset.
	//
	struct FDiffResult
	{
		FString ChangedAssetObjectPath;
		FString TagBaseValue;
		FString TagCurrentValue;
	};

	TMap<FName /* TagName */, TMap<FTopLevelAssetPath /* AssetClass */, TArray<FDiffResult>>> Results;
	TMap<FTopLevelAssetPath /* AssetClass */, TArray<FName /* PackageName */>> NoTagPackagesByAssumedClass;
	TArray<FName /* PackageName */> PackagesWithUnassignableDiffsAndUntaggedAssets;
	TMap<FTopLevelAssetPath /* AssetClass */, TArray<FName /* PackageName */>> PackagesWithUnassignableDiffsByAssumedClass;
	
	for (const FName& ChangedPackageName : PackagesWithChangedChunks)
	{
		TArray<FAssetData const*> BaseAssetDatas = BaseState.CopyAssetsByPackageName(ChangedPackageName);
		TArray<FAssetData const*> CurrentAssetDatas = CurrentState.CopyAssetsByPackageName(ChangedPackageName);

		struct FDiffTag
		{
			// Order is used to sort the diff blame keys so that the correct thing is blamed. This is
			// so that e.g. changing the texture source (which would change the ddc key) gets properly blamed
			// as it is lower order.
			int Order;
			FName TagName;
			FString BaseValue;
			FString CurrentValue;

			const FAssetData* BaseAssetData;
			const FAssetData* CurrentAssetData;
		};
		
		// We want to find all the tags that are in both base/current.
		TMap<FName /* AssetName */, TArray<FDiffTag>> PackageDiffTags;
		bool bPackageHasUntaggedAsset = false;
		for (const FAssetData* BaseAssetData : BaseAssetDatas)
		{
			BaseAssetData->EnumerateTags([&PackageDiffTags, BaseAssetData, CurrentAssetDatas](TPair<FName, FAssetTagValueRef> TagAndValue)
			{
				TCHAR Name[NAME_SIZE];
				TagAndValue.Key.GetPlainNameString(Name);
				if (FCString::Strncmp(Name, TEXT("Cook_Diff_"), 10))
				{
					return;
				}

				// This is O(N) but like 99.9% of the time there's only 1 asset.
				const FAssetData* const* CurrentAssetData = CurrentAssetDatas.FindByPredicate([SearchAssetName = &BaseAssetData->AssetName](const FAssetData* AssetData) { return (AssetData->AssetName == *SearchAssetName); });
				if (CurrentAssetData == nullptr)
				{
					return;
				}

				FString CurrentValue;
				if (CurrentAssetData[0]->GetTagValue(TagAndValue.Key, CurrentValue) == false)
				{
					// Both version don't have the tag so we can't compare.
					return;
				}

				TArray<FDiffTag>& AssetDiffTags = PackageDiffTags.FindOrAdd(BaseAssetData->AssetName);
				FDiffTag& Tag = AssetDiffTags.AddDefaulted_GetRef();
				Tag.Order = FCString::Atoi(Name + FCString::Strlen(TEXT("Cook_Diff_"))); // this gets optimized to +10
				Tag.TagName = TagAndValue.Key;
				Tag.BaseValue =  TagAndValue.Value.AsString();
				Tag.CurrentValue = MoveTemp(CurrentValue);
				Tag.BaseAssetData = BaseAssetData;
				Tag.CurrentAssetData = *CurrentAssetData;
			});

			if (PackageDiffTags.Contains(BaseAssetData->AssetName) == false)
			{
				bPackageHasUntaggedAsset = true;
				// An asset exists in the package that doesn't have any tags - make a note so that
				// we can suggest this caused the bulk data diff if we don't find a blame.
			}
		}

		bool bPackageHasUntaggedAndTaggedAssets = false;
		if (PackageDiffTags.Num())
		{
			if (bPackageHasUntaggedAsset)
			{
				bPackageHasUntaggedAndTaggedAssets = true;
			}
		}
		else
		{
			// Nothing has anything to use for diff blaming for this package.
			// Try to find a representative asset class from the assets in the package.
			FAssetData const* RepresentativeAsset = UE::AssetRegistry::GetMostImportantAsset(CurrentAssetDatas, UE::AssetRegistry::EGetMostImportantAssetFlags::RequireOneTopLevelAsset);
			if (RepresentativeAsset == nullptr)
			{
				NoTagPackagesByAssumedClass.FindOrAdd(FTopLevelAssetPath()).Add(ChangedPackageName);
			}
			else
			{
				NoTagPackagesByAssumedClass.FindOrAdd(RepresentativeAsset->AssetClassPath).Add(ChangedPackageName);
			}
			
			continue;
		}

		// Now we check and see if any of the diff tags can tell us why the package changed.
		// We could find multiple assets that caused the change.
		bool bFoundDiffTag = false;
		for (TPair<FName, TArray<FDiffTag>>& AssetDiffTagPair : PackageDiffTags)
		{
			TArray<FDiffTag>& AssetDiffTags = AssetDiffTagPair.Value;
			Algo::SortBy(AssetDiffTags, &FDiffTag::Order);
			
			for (FDiffTag& Tag : AssetDiffTags)
			{
				if (Tag.BaseValue != Tag.CurrentValue)
				{
					TMap<FTopLevelAssetPath, TArray<FDiffResult>>& TagResults = Results.FindOrAdd(Tag.TagName);

					TArray<FDiffResult>& ClassResults = TagResults.FindOrAdd(Tag.BaseAssetData->AssetClassPath);

					FDiffResult& Result = ClassResults.AddDefaulted_GetRef();
					Result.ChangedAssetObjectPath = Tag.BaseAssetData->GetObjectPathString();
					Result.TagBaseValue = MoveTemp(Tag.BaseValue);
					Result.TagCurrentValue = MoveTemp(Tag.CurrentValue);
					bFoundDiffTag = true;
					break;
				}
			}
		}


		if (bFoundDiffTag == false)
		{
			// This means that all the tags they added didn't change, but the asset did.
			// Assuming that a DDC key tag has been added, this means either:
			// 
			// A) The asset changed independent of DDC key, which is a build consistency / determinism alert.
			// B) The package had an asset with tags and an asset without tags, and the asset without tags caused
			//	  the bulk data change.
			//
			// Unfortunately A) is a Big Deal and needs a warning, but B might end up being common due to blueprint classes,
			// so we segregate the lists.
			if (bPackageHasUntaggedAndTaggedAssets)
			{
				PackagesWithUnassignableDiffsAndUntaggedAssets.Add(ChangedPackageName);
			}
			else
			{
				FAssetData const* RepresentativeAsset = UE::AssetRegistry::GetMostImportantAsset(CurrentAssetDatas, UE::AssetRegistry::EGetMostImportantAssetFlags::RequireOneTopLevelAsset);
				if (RepresentativeAsset == nullptr)
				{
					PackagesWithUnassignableDiffsByAssumedClass.FindOrAdd(FTopLevelAssetPath()).Add(ChangedPackageName);
				}
				else
				{
					PackagesWithUnassignableDiffsByAssumedClass.FindOrAdd(RepresentativeAsset->AssetClassPath).Add(ChangedPackageName);
				}
			}
		}
	}

	auto ProcessPackageClassAndSize = [](FAssetRegistryState& State, const FName& PackageName, uint64& SizeToUpdate, TMap<FTopLevelAssetPath, TArray<FName>>& PackagesByClassToUpdate)
	{
		const FAssetData* MIAsset = UE::AssetRegistry::GetMostImportantAsset(State.CopyAssetsByPackageName(PackageName), UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
		if (MIAsset)
		{
			// IoStoreUtilities puts the size of the package on the most important asset
			uint64 CurrentCompressedSize = 0;
			if (MIAsset->GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CurrentCompressedSize))
			{
				SizeToUpdate += CurrentCompressedSize;
			}

			PackagesByClassToUpdate.FindOrAdd(MIAsset->AssetClassPath).Add(PackageName);
		}
	};

	auto SumPackageSizes = [&PackageSizes](const TArray<FName>& PackageList, bool bUseBaseSize)
	{
		uint64 Total = 0;
		for (const FName& PackageName : PackageList)
		{
			FPackageSizes* Sizes = PackageSizes.Find(PackageName);
			if (Sizes)
			{
				Total += bUseBaseSize ? Sizes->BaseCompressedSize : Sizes->CurrentCompressedSize;
			}
		}
		return Total;
	};

	TMap<FTopLevelAssetPath, TArray<FName>> NewPackagesByClass;
	uint64 TotalNewPackagesSize = 0;
	for (const TPair<FName, TArray<FIoHash, TInlineAllocator<1>>>& PackageHashesPair : PackagesWithNewChunks)
	{
		ProcessPackageClassAndSize(CurrentState, PackageHashesPair.Key, TotalNewPackagesSize, NewPackagesByClass);
	}

	TMap<FTopLevelAssetPath, TArray<FName>> DeletedPackagesByClass;
	uint64 TotalDeletedPackagesSize = 0;
	for (const FName& DeletedPackage : PackagesWithDeletedChunks)
	{
		ProcessPackageClassAndSize(BaseState, DeletedPackage, TotalDeletedPackagesSize, DeletedPackagesByClass);
	}

	TMap<FTopLevelAssetPath, TArray<FName>> MovedPackagesByClass;
	uint64 TotalMovedPackagesSize = 0;
	for (const TPair<FName, FName>& MovedPackageFromTo : MovedPackagesFromTo)
	{
		ProcessPackageClassAndSize(BaseState, MovedPackageFromTo.Key, TotalMovedPackagesSize, MovedPackagesByClass);
	}

	int32 PackagesWithNoSize = UnionedPackages.Num() - PackageSizes.Num();
	
	UE_LOGF(LogDiffAssetBulk, Display, "    =====================================================");
	if (bEvaluateOptional)
	{
		UE_LOGF(LogDiffAssetBulk, Display, "    OPTIONAL bulk data only");
	}
	else
	{
		UE_LOGF(LogDiffAssetBulk, Display, "    Excluding OPTIONAL bulk data chunks");
	}

	UE_LOGF(LogDiffAssetBulk, Display, "");

	UE_LOGF(LogDiffAssetBulk, Display, "    Base Packages:                %8d %17ls bytes", BasePackages.Num(), *FText::AsNumber(BaseTotalSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Current Packages:             %8d %17ls bytes", CurrentPackages.Num(), *FText::AsNumber(CurrentTotalSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Bulk Data Packages Added:     %8d %17ls bytes", PackagesWithNewChunks.Num(), *FText::AsNumber(TotalNewPackagesSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Bulk Data Packages Deleted:   %8d %17ls bytes", PackagesWithDeletedChunks.Num(), *FText::AsNumber(TotalDeletedPackagesSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Bulk Data Packages Moved:     %8d %17ls bytes", MovedPackagesFromTo.Num(), *FText::AsNumber(TotalMovedPackagesSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Bulk Data Packages Changed:   %8d %17ls bytes (all chunks!)", PackagesWithChangedChunks.Num(), *FText::AsNumber(TotalChangedSize).ToString());
	UE_LOGF(LogDiffAssetBulk, Display, "    Packages with no size info:   %8d", PackagesWithNoSize);
	UE_LOGF(LogDiffAssetBulk, Display, "");

	if (PackagesWithChangedChunks.Num())
	{
		TArray<FName>& CantDetermineAssetClassPackages = NoTagPackagesByAssumedClass.FindOrAdd(FTopLevelAssetPath());

		// Note this output is parsed by build scripts, be sure to fix those up if you change anything here.
		UE_LOGF(LogDiffAssetBulk, Display, "Changed package breakdown:                               // -ListNoBlame=<class name>");
		UE_LOGF(LogDiffAssetBulk, Display, "    No blame information available:");
		{
			Algo::Sort(CantDetermineAssetClassPackages, FNameLexicalLess());
			UE_LOGF(LogDiffAssetBulk, Display, "        Unknown                               %6d     // Couldn't pick a representative asset in the package. -ListUnrepresented", CantDetermineAssetClassPackages.Num());
			if (bListUnrepresented)
			{
				for (const FName& PackageName : CantDetermineAssetClassPackages)
				{
					UE_LOGF(LogDiffAssetBulk, Display, "            %ls", *PackageName.ToString());
				}
			}
			if (ChangedCSVAr.IsValid())
			{
				for (const FName& PackageName : CantDetermineAssetClassPackages)
				{
					ChangedCSVAr->Logf(TEXT("NoBlameInfo, Unknown, %s,,"), *WriteToString<64>(PackageName));
				}
			}
		}
		for (TPair<FTopLevelAssetPath, TArray<FName>>& ClassPackages : NoTagPackagesByAssumedClass)
		{
			if (ClassPackages.Key == FTopLevelAssetPath()) // Skip packages we couldn't find a class for, handled above.
			{
				continue;
			}

			uint64 TotalSizes = SumPackageSizes(ClassPackages.Value, false);

			TStringBuilder<64> ClassName;
			ClassName << ClassPackages.Key;

			UE_LOGF(LogDiffAssetBulk, Display, "        %-37ls %6d %17ls bytes", *ClassName, ClassPackages.Value.Num(), *FText::AsNumber(TotalSizes).ToString());
			if (ListNoBlame.Compare(TEXT("All"), ESearchCase::IgnoreCase) == 0 ||
				ListNoBlame.Compare(ClassPackages.Key.ToString(), ESearchCase::IgnoreCase) == 0)
			{
				for (const FName& PackageName : ClassPackages.Value)
				{
					UE_LOGF(LogDiffAssetBulk, Display, "        %ls", *WriteToString<64>(PackageName));
				}
			}
			if (ChangedCSVAr.IsValid())
			{
				for (const FName& PackageName : ClassPackages.Value)
				{
					FPackageSizes* Sizes = PackageSizes.Find(PackageName);

					ChangedCSVAr->Logf(TEXT("NoBlameInfo, %s, %s,,,%s,%s,%s,%s"), *ClassName, *WriteToString<64>(PackageName), 
						Sizes ? *WriteToString<32>(Sizes->BaseCompressedSize) : TEXT(""), 
						Sizes ? *WriteToString<32>(Sizes->CurrentCompressedSize) : TEXT(""),
						Sizes ? *WriteToString<32>(Sizes->BaseUncompressedSize) : TEXT(""),
						Sizes ? *WriteToString<32>(Sizes->CurrentUncompressedSize) : TEXT(""));
				}
			}
		}



		if (PackagesWithUnassignableDiffsByAssumedClass.Num())
		{
			int32 TotalUnassignablePackages = 0;
			for (TPair<FTopLevelAssetPath, TArray<FName>>& ClassPackages : PackagesWithUnassignableDiffsByAssumedClass)
			{
				TotalUnassignablePackages += ClassPackages.Value.Num();
			}
		

			UE_LOGF(LogDiffAssetBulk, Display, "    Can't determine blame:                    %6d     // Assets had blame tags but all matched - check determinism! -ListDeterminism", TotalUnassignablePackages);
			for (TPair<FTopLevelAssetPath, TArray<FName>>& ClassPackages : PackagesWithUnassignableDiffsByAssumedClass)
			{
				uint64 TotalSizes = SumPackageSizes(ClassPackages.Value, false);

				TStringBuilder<64> ClassName;
				ClassName << ClassPackages.Key;

				UE_LOGF(LogDiffAssetBulk, Display, "        %-37ls %6d %17ls bytes", *ClassName, ClassPackages.Value.Num(), *FText::AsNumber(TotalSizes).ToString());
				Algo::Sort(ClassPackages.Value, FNameLexicalLess());
				if (bListDeterminism)
				{
					for (const FName& PackageName : ClassPackages.Value)
					{
						UE_LOGF(LogDiffAssetBulk, Display, "            %ls", *WriteToString<64>(PackageName));
					}
				}
				if (ChangedCSVAr.IsValid())
				{
					for (const FName& PackageName : ClassPackages.Value)
					{
						ChangedCSVAr->Logf(TEXT("NonDetermistic, %s, %s,,"), *ClassName, *WriteToString<64>(PackageName));
					}
				}
			}
		}

		if (PackagesWithUnassignableDiffsAndUntaggedAssets.Num())
		{
			Algo::Sort(PackagesWithUnassignableDiffsAndUntaggedAssets, FNameLexicalLess());
		
			UE_LOGF(LogDiffAssetBulk, Display, "    Potential untagged assets:          %6d     // Package had assets with blame tags that matched, but also untagged assets. Might be determinism! -ListMixed", PackagesWithUnassignableDiffsAndUntaggedAssets.Num());
			if (bListMixed)
			{
				for (const FName& PackageName : PackagesWithUnassignableDiffsAndUntaggedAssets)
				{
					UE_LOGF(LogDiffAssetBulk, Display, "        %ls", *PackageName.ToString());
				}
			}
			if (ChangedCSVAr.IsValid())
			{
				for (const FName& PackageName : PackagesWithUnassignableDiffsAndUntaggedAssets)
				{
					ChangedCSVAr->Logf(TEXT("Mixed, Unknown, %s,,"), *WriteToString<64>(PackageName));
				}
			}
		}

		if (Results.Num())
		{
			UE_LOGF(LogDiffAssetBulk, Display, "    Summary changes by blame tag:                        // -ListBlame=<BlameTag>");

			for (TPair<FName, TMap<FTopLevelAssetPath, TArray<FDiffResult>>>& TagResults : Results)
			{
				uint32 TagCount = 0;
				for (TPair<FTopLevelAssetPath, TArray<FDiffResult>>& ClassResults : TagResults.Value)
				{
					TagCount += ClassResults.Value.Num();
				}

				TStringBuilder<32> TagName;
				TagName << TagResults.Key;

				const TCHAR** TagHelp = BuiltinDiffTagHelpMap.Find(TagResults.Key);
				if (TagHelp != nullptr)
				{
					UE_LOGF(LogDiffAssetBulk, Display, "        %-37ls %6d     // %ls", *TagName, TagCount, *TagHelp);
				}
				else
				{
					UE_LOGF(LogDiffAssetBulk, Display, "        %-37ls %6d", *TagName, TagCount);
				}

				bool bListing = FCString::Stricmp(*ListBlame, TEXT("All")) == 0 ||
					FCString::Stricmp(*ListBlame, *TagName) == 0;

				for (TPair<FTopLevelAssetPath, TArray<FDiffResult>>& ClassResults : TagResults.Value)
				{
					Algo::SortBy(ClassResults.Value, &FDiffResult::ChangedAssetObjectPath);

					if (bListing)
					{
						for (FDiffResult& Result : ClassResults.Value)
						{
							UE_LOGF(LogDiffAssetBulk, Display, "                %ls [%ls -> %ls]", *Result.ChangedAssetObjectPath, *Result.TagBaseValue, *Result.TagCurrentValue);
						}
					}
					if (ChangedCSVAr.IsValid())
					{
						for (FDiffResult& Result : ClassResults.Value)
						{
							ChangedCSVAr->Logf(TEXT("%s, %s, %s, %s, %s"), *TagResults.Key.ToString(), *ClassResults.Key.ToString(), *WriteToString<64>(Result.ChangedAssetObjectPath), *Result.TagBaseValue, *Result.TagCurrentValue);
						}
					}
				}
			}
		}
	} // end changed packages
	
	auto ProcessPackagesByClass = [&SumPackageSizes](const TMap<FTopLevelAssetPath, TArray<FName>>& PackagesByClass, FArchive* CSVArchive, const TMap<FName, FName>* PackageDestinationIfMoved, bool bUseBaseSizes)
	{
		for (const TPair<FTopLevelAssetPath, TArray<FName>>& PackagesForClass : PackagesByClass)
		{
			TStringBuilder<64> ClassName;
			ClassName << PackagesForClass.Key;

			uint64 TotalSize = SumPackageSizes(PackagesForClass.Value, bUseBaseSizes);

			UE_LOGF(LogDiffAssetBulk, Display, "    %-37ls %6d %17ls bytes", *ClassName, PackagesForClass.Value.Num(), *FText::AsNumber(TotalSize).ToString());

			if (CSVArchive)
			{
				for (const FName& PackageName : PackagesForClass.Value)
				{
					if (PackageDestinationIfMoved)
					{
						CSVArchive->Logf(TEXT("%s, %s, %s"), *ClassName, *WriteToString<64>(PackageName), *WriteToString<64>(*PackageDestinationIfMoved->Find(PackageName)));
					}
					else
					{
						CSVArchive->Logf(TEXT("%s, %s"), *ClassName, *WriteToString<64>(PackageName));
					}
				}
			}
		}
	};

	if (PackagesWithNewChunks.Num())
	{
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "New package breakdown:");
		ProcessPackagesByClass(NewPackagesByClass, NewCSVAr.Get(), nullptr, false);
	}

	if (PackagesWithDeletedChunks.Num())
	{
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "Deleted package breakdown:");
		ProcessPackagesByClass(DeletedPackagesByClass, DeletedCSVAr.Get(), nullptr, true);
	}

	if (MovedPackagesFromTo.Num())
	{
		UE_LOGF(LogDiffAssetBulk, Display, "");
		UE_LOGF(LogDiffAssetBulk, Display, "Moved package breakdown:");
		ProcessPackagesByClass(MovedPackagesByClass, MovedCSVAr.Get(), &MovedPackagesFromTo, true);
	}


	UE_LOGF(LogDiffAssetBulk, Display, "Done.");

	return 0;
}


INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	// start up the main loop
	GEngineLoop.PreInit(ArgC, ArgV);

	double StartTime = FPlatformTime::Seconds();

	int32 Result = RunDiffAssetBulkData();

	UE_LOGF(LogDiffAssetBulk, Display, "Logging..");

	GLog->Flush();

	RequestEngineExit(TEXT("DiffAssetBulkData Exiting"));

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return Result;
}

